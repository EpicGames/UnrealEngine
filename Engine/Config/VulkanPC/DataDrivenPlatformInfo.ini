; Have a single source of truth for all desktop class Vulkan platforms

; This is done so all the platforms using SP_VULKAN_SM5/SM6 can share the same DDC
; If your Vulkan platform needs its own settings, create a new SP for it

[DataDrivenPlatformInfo]
bIsConfidential=false
bIsFakePlatform=true
GlobalIdentifier=97F8C85EF1B0424792D201C893358C8F

[ShaderPlatform VULKAN_SM5]
Language=Vulkan
MaxFeatureLevel=SM5
ShaderFormat=SF_VULKAN_SM5
bSupportsSpecializationConstants = true

bIsMobile = false
bIsMetalMRT = false
bIsPC = true
bIsConsole = false
bIsAndroidOpenGLES = false
bSupportsDxc = true
bIsSPIRV=true

bSupportsDebugViewShaders=true

; //#todo-vulkanddpi
bSupportsMobileMultiView = false

; //#todo-vulkanddpi
bSupportsArrayTextureCompression = false

bSupportsDistanceFields = true
bSupportsDiaphragmDOF = true
bSupportsRGBColorBuffer = true
bSupportsPercentageCloserShadows=true
bSupportsIndexBufferUAVs = true

bSupportsInstancedStereo = true
SupportsMultiViewport = RuntimeDependent

bSupportsMSAA = true

; //#todo-vulkanddpi
bSupports4ComponentUAVReadWrite = false

; //#todo-vulkanddpi
bSupportsRenderTargetWriteMask = false

bSupportsRayTracing = false
bSupportsRayTracingShaders = false
bSupportsInlineRayTracing = false
bSupportsRayTracingIndirectInstanceData = true

bSupportsGPUScene = true

; //#todo-vulkanddpi
bSupportsPrimitiveShaders = false

bSupportsUInt64ImageAtomics = false

; //#todo-vulkanddpi
bRequiresVendorExtensionsForAtomics = false

bSupportsNanite = false
bSupportsSceneDataCompressedTransforms = true
bSupportsLumenGI = true

; //#todo-vulkanddpi
bSupportsSSDIndirect = false

; //#todo-vulkanddpi
bSupportsTemporalHistoryUpscale = false

; //#todo-vulkanddpi
bSupportsRTIndexFromVS = false

; //#todo-vulkanddpi
bSupportsIntrinsicWaveOnce = false

; //#todo-vulkanddpi
bSupportsConservativeRasterization = false

bSupportsWaveOperations=RuntimeDependent
MinimumWaveSize=4
MaximumWaveSize=128

; //#todo-vulkanddpi
bRequiresExplicit128bitRT = false

bTargetsTiledGPU = false
bNeedsOfflineCompiler = false

bSupportsComputeFramework = true

; //#todo-vulkanddpi
bSupportsDualSourceBlending = true

; //#todo-vulkanddpi
bRequiresGeneratePrevTransformBuffer = false

bRequiresRenderTargetDuringRaster = true

; //#todo-vulkanddpi
bRequiresDisableForwardLocalLights = true

bCompileSignalProcessingPipeline = true
bSupportsGen5TemporalAA=true

; //#todo-vulkanddpi
bSupportsPerPixelDBufferMask = false


bSupportsVariableRateShading = true

NumberOfComputeThreads = 64

bSupportsHairStrandGeometry=true
bSupportsFFTBloom=true
bSupportsDiaphragmDOF=true

; The support really is conditional, the runtime code needs to also test GRHISupportsArrayIndexFromAnyShader
bSupportsVertexShaderLayer=true

bSupportsAnisotropicMaterials=true

; disable until DXC -> SPRIV ROV support is implemented
bSupportsROV=false
bSupportsOIT=false

EnablesHLSL2021ByDefault=1
FriendlyName=LOCTEXT("FriendlyShaderPlatformName_VulkanPC_SM5", "PC Vulkan SM5")

bSupportsNNEShaders=false
bSupportsUniformBufferObjects = true
bSupportsRealTypes=Unsupported
bSupportsIndependentSamplers=true



[ShaderPlatform VULKAN_SM6]
Language=Vulkan
MaxFeatureLevel=SM6
ShaderFormat=SF_VULKAN_SM6

bIsMobile = false
bIsMetalMRT = false
bIsPC = true
bIsConsole = false
bIsAndroidOpenGLES = false
bSupportsDxc = true
bIsSPIRV=true
bSupportsSpecializationConstants = true

bSupportsDebugViewShaders=true
bSupportsMobileMultiView = false
bSupportsArrayTextureCompression = false

bSupportsDistanceFields = true
bSupportsDiaphragmDOF = true
bSupportsRGBColorBuffer = true
bSupportsPercentageCloserShadows=true
bSupportsVolumetricFog = true
bSupportsIndexBufferUAVs = true

bSupportsInstancedStereo = true
SupportsMultiViewport = RuntimeDependent

bSupportsMSAA = true
bSupports4ComponentUAVReadWrite = false
bSupportsRenderTargetWriteMask = false

bSupportsRayTracing = true
bSupportsInlineRayTracing = true
bSupportsRayTracingIndirectInstanceData = true

; Enable full Vulkan ray tracing
bSupportsRayTracingShaders = true
bSupportsPathTracing = true
bSupportsRayTracingCallableShaders = true
bSupportsRayTracingProceduralPrimitive = true
bSupportsRayTracingLinearSweptSpheres = true
StaticShaderBindingLayoutSupport=RayTracingOnly

bSupportsGPUScene = true
bSupportsPrimitiveShaders = false
bSupportsUInt64ImageAtomics = true
bRequiresVendorExtensionsForAtomics = false

bSupportsNanite = true
bSupportsSceneDataCompressedTransforms = true
bSupportsLumenGI = true

bSupportsMeshShadersTier0=true
bSupportsMeshShadersTier1=true
MaxMeshShaderThreadGroupSize=128

; //#todo-vulkanddpi
bSupportsSSDIndirect = false

; //#todo-vulkanddpi
bSupportsTemporalHistoryUpscale = false

; //#todo-vulkanddpi
bSupportsRTIndexFromVS = false

; //#todo-vulkanddpi
bSupportsIntrinsicWaveOnce = false

; //#todo-vulkanddpi
bSupportsConservativeRasterization = false

bSupportsWaveOperations=RuntimeGuaranteed
MinimumWaveSize=4
MaximumWaveSize=128

; //#todo-vulkanddpi
bRequiresExplicit128bitRT = false

bTargetsTiledGPU = false
bNeedsOfflineCompiler = false

bSupportsComputeFramework = true

; //#todo-vulkanddpi
bSupportsDualSourceBlending = true

; //#todo-vulkanddpi
bRequiresGeneratePrevTransformBuffer = false

bRequiresRenderTargetDuringRaster = true

bCompileSignalProcessingPipeline = true
bSupportsGen5TemporalAA=true

; //#todo-vulkanddpi
bSupportsPerPixelDBufferMask = false

bSupportsVariableRateShading = true

NumberOfComputeThreads = 64

bSupportsHairStrandGeometry=true
bSupportsFFTBloom=true
bSupportsDiaphragmDOF=true

bSupportsVertexShaderLayer=true

bSupportsAnisotropicMaterials=true

bSupportsAdaptiveGBuffer=true

; disable until DXC -> SPRIV ROV support is implemented
bSupportsROV=false
bSupportsOIT=false

EnablesHLSL2021ByDefault=1
FriendlyName=LOCTEXT("FriendlyShaderPlatformName_VulkanPC_SM6", "PC Vulkan SM6")

bSupportsBindless=true
bSupportsNNEShaders=false
bSupportsUniformBufferObjects = true
bSupportsRealTypes=RuntimeDependent
bSupportsIndependentSamplers=true

SupportsBarycentricsSemantic=RuntimeGuaranteed


[ShaderPlatform VULKAN_PCES3_1]
Language=Vulkan
MaxFeatureLevel=ES3_1
ShaderFormat=SF_VULKAN_ES31
bIsPC=true
bIsMobile=true
bSupportsSpecializationConstants = true
bSupportsDebugViewShaders=true
; Enables RDG pass merging(like on Android), helps with debugging
bTargetsTiledGPU=true
bSupportsMobileMultiView=true
bSupportsIndexBufferUAVs=true
bSupportsManualVertexFetch=false
; The support really is conditional, the runtime code needs to also test GRHISupportsArrayIndexFromAnyShader
bSupportsVertexShaderLayer=true
bSupportsSceneDataCompressedTransforms=true
bSupportsShaderPipelines = false
bSupportsVariableRateShading=true
bSupportsVertexShaderSRVs = false
bSupportsUniformBufferObjects = true
bSupportsDualSourceBlending = true
bSupportsIndependentSamplers=true
bSupportsComputeFramework = true
bSupportsDistanceFields = true
FriendlyName=LOCTEXT("FriendlyShaderPlatformName_VulkanPC_Mobile", "PC Vulkan Mobile")

[PreviewPlatform VULKAN_SM5]
PlatformName=PC
ShaderFormat=SF_VULKAN_SM5
ShaderPlatform=VULKAN_SM5
MenuTooltip=LOCTEXT("PreviewMenuTooltip_VulkanPC_SM5", "Linux using SM5 profile")

[PreviewPlatform VULKAN_SM6]
PlatformName=PC
ShaderFormat=SF_VULKAN_SM6
ShaderPlatform=VULKAN_SM6
MenuTooltip=LOCTEXT("PreviewMenuTooltip_VulkanPC_SM6", "Linux using SM6 profile")
